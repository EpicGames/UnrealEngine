// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blueprint/UserWidget.h"
#include "Components/TextBlock.h"
#include "Components/Image.h"
#include "Components/Widget.h"
#include "Extensions/UIComponent.h"
#include "UMGToolSetTestFixtures.generated.h"

/**
 * Base UIComponent used to test hierarchy-conflict error reporting in AddUIComponent.
 * UTestDerivedUIComponent inherits from this class.
 */
UCLASS()
class UTestBaseUIComponent : public UUIComponent
{
	GENERATED_BODY()
};

/**
 * Derived UIComponent used alongside UTestBaseUIComponent to trigger the
 * "component in the same class hierarchy" error path in AddUIComponent.
 */
UCLASS()
class UTestDerivedUIComponent : public UTestBaseUIComponent
{
	GENERATED_BODY()
};

/**
 * Test parent widget with BindWidget properties for generic testing.
 * Used by BindWidget and CompileErrors test groups to validate toolset
 * behavior with parent classes that have BindWidget requirements,
 * without depending on any game-specific classes.
 */
UCLASS()
class UUMGTestWidgetWithBindings : public UUserWidget
{
	GENERATED_BODY()

public:
	/** Mandatory BindWidget — compiler errors if not present in child blueprint. */
	UPROPERTY(meta = (BindWidget))
	TObjectPtr<UTextBlock> RequiredText = nullptr;

	/** Mandatory BindWidget — compiler errors if not present in child blueprint. */
	UPROPERTY(meta = (BindWidget))
	TObjectPtr<UImage> RequiredImage = nullptr;

	/** Optional BindWidget — compiler notes but does not error. */
	UPROPERTY(meta = (BindWidgetOptional))
	TObjectPtr<UImage> OptionalIcon = nullptr;

	/** Regular property (not BindWidget) for name-collision testing. */
	UPROPERTY()
	TObjectPtr<UWidget> InternalRef = nullptr;
};

