// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_DEV_AUTOMATION_TESTS

#include "Engine/Engine.h"
#include "Misc/AutomationTest.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"

namespace UMGToolSetTest
{
	const auto Flags =
		EAutomationTestFlags::EditorContext |
		EAutomationTestFlags::ProductFilter |
		EAutomationTestFlags::CriticalPriority;

	/** Shared mount point for transient test assets. */
	static const FString TestMountPoint = TEXT("/Automation/UMGToolSetTest/");

	/** Registers a transient mount point for test assets. */
	inline void RegisterTestMountPoint()
	{
		FPackageName::RegisterMountPoint(*TestMountPoint, FPaths::AutomationTransientDir());
	}

	/** Unregisters the transient mount point and forces GC. */
	inline void UnregisterTestMountPoint()
	{
		FPackageName::UnRegisterMountPoint(*TestMountPoint, FPaths::AutomationTransientDir());
		if (GEngine)
		{
			GEngine->ForceGarbageCollection(true);
		}
	}
}

#endif // WITH_DEV_AUTOMATION_TESTS
