// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#include "UMGToolSetTestFlags.h"
#include "UMGToolSet.h"
#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "Components/VerticalBox.h"
#include "Components/VerticalBoxSlot.h"
#include "Components/TextBlock.h"
#include "Components/Border.h"
#include "Components/ExpandableArea.h"
#include "Blueprint/UserWidget.h"
#include "Blueprint/WidgetTree.h"
#include "WidgetBlueprint.h"
#include "WidgetBlueprintOperationUtils.h"

BEGIN_DEFINE_SPEC(
	FWidgetDescriptionTest,
	"AI.Toolsets.UMGToolSet.WidgetDescription",
	UMGToolSetTest::Flags)

UWidgetBlueprint* TestBP = nullptr;
int32 TestCounter = 0;

void BuildFixture();
void TeardownFixture();

END_DEFINE_SPEC(FWidgetDescriptionTest)

void FWidgetDescriptionTest::BuildFixture()
{
	FString BPName = FString::Printf(TEXT("WBP_WalkerTest_%d"), TestCounter++);
	TestBP = UUMGToolSet::CreateWidgetBlueprint(
		UMGToolSetTest::TestMountPoint, BPName,
		UUserWidget::StaticClass());

	UBorder* Root = TestBP->WidgetTree->ConstructWidget<UBorder>(UBorder::StaticClass(), TEXT("Root"));
	UVerticalBox* Stack = TestBP->WidgetTree->ConstructWidget<UVerticalBox>(UVerticalBox::StaticClass(), TEXT("Stack"));
	UTextBlock* First = TestBP->WidgetTree->ConstructWidget<UTextBlock>(UTextBlock::StaticClass(), TEXT("First"));
	UTextBlock* Second = TestBP->WidgetTree->ConstructWidget<UTextBlock>(UTextBlock::StaticClass(), TEXT("Second"));

	Root->SetContent(Stack);
	Stack->AddChildToVerticalBox(First);
	Stack->AddChildToVerticalBox(Second);

	TestBP->WidgetTree->RootWidget = Root;
}

void FWidgetDescriptionTest::TeardownFixture()
{
	TestBP = nullptr;
}

void FWidgetDescriptionTest::Define()
{
	Describe("Walker", [this]()
	{
		BeforeEach([this]()
		{
			UMGToolSetTest::RegisterTestMountPoint();
			BuildFixture();
		});
		AfterEach([this]()
		{
			TeardownFixture();
			UMGToolSetTest::UnregisterTestMountPoint();
		});

		It("walks depth-first in child-slot order", [this]()
		{
			TArray<TPair<FName, int32>> Visited;
			FWidgetBlueprintOperationUtils::WalkWidgetTree(
				TestBP, nullptr, -1,
				[&](const UWidget* W, int32 Depth, FName /*SlotName*/)
				{
					Visited.Add({W->GetFName(), Depth});
				});

			TestEqual(TEXT("widget count"), Visited.Num(), 4);
			TestEqual(TEXT("order[0]"), Visited[0].Key, FName(TEXT("Root")));
			TestEqual(TEXT("order[1]"), Visited[1].Key, FName(TEXT("Stack")));
			TestEqual(TEXT("order[2]"), Visited[2].Key, FName(TEXT("First")));
			TestEqual(TEXT("order[3]"), Visited[3].Key, FName(TEXT("Second")));
		});

		It("respects MaxDepth=0 (only StartWidget)", [this]()
		{
			TArray<FName> Visited;
			FWidgetBlueprintOperationUtils::WalkWidgetTree(
				TestBP, nullptr, 0,
				[&](const UWidget* W, int32 /*Depth*/, FName /*SlotName*/)
				{
					Visited.Add(W->GetFName());
				});

			TestEqual(TEXT("widget count"), Visited.Num(), 1);
			TestEqual(TEXT("only root"), Visited[0], FName(TEXT("Root")));
		});

		It("respects MaxDepth=1 (root + immediate children)", [this]()
		{
			TArray<FName> Visited;
			FWidgetBlueprintOperationUtils::WalkWidgetTree(
				TestBP, nullptr, 1,
				[&](const UWidget* W, int32 /*Depth*/, FName /*SlotName*/)
				{
					Visited.Add(W->GetFName());
				});

			TestEqual(TEXT("widget count"), Visited.Num(), 2);
		});

		It("respects StartWidget (subtree only)", [this]()
		{
			UWidget* Stack = TestBP->WidgetTree->FindWidget(FName(TEXT("Stack")));
			TestNotNull(TEXT("Stack found"), Stack);

			TArray<FName> Visited;
			FWidgetBlueprintOperationUtils::WalkWidgetTree(
				TestBP, Stack, -1,
				[&](const UWidget* W, int32 /*Depth*/, FName /*SlotName*/)
				{
					Visited.Add(W->GetFName());
				});

			TestEqual(TEXT("widget count"), Visited.Num(), 3);  // Stack + First + Second
			TestEqual(TEXT("order[0]"), Visited[0], FName(TEXT("Stack")));
		});

		It("ComputeWidgetTreeDepth returns 2 for fixture", [this]()
		{
			int32 Depth = FWidgetBlueprintOperationUtils::ComputeWidgetTreeDepth(TestBP);
			TestEqual(TEXT("depth"), Depth, 2);  // Border -> VerticalBox -> TextBlock = 2 levels below root
		});

		It("returns empty walk for WBP with no root widget", [this]()
		{
			FString EmptyName = FString::Printf(TEXT("WBP_Empty_%d"), TestCounter++);
			UWidgetBlueprint* EmptyBP = UUMGToolSet::CreateWidgetBlueprint(
				UMGToolSetTest::TestMountPoint, EmptyName,
				UUserWidget::StaticClass());
			TestNotNull(TEXT("EmptyBP created"), EmptyBP);
			// RootWidget intentionally null

			int32 VisitCount = 0;
			FWidgetBlueprintOperationUtils::WalkWidgetTree(
				EmptyBP, nullptr, -1,
				[&](const UWidget* /*W*/, int32 /*Depth*/, FName /*SlotName*/)
				{
					++VisitCount;
				});

			TestEqual(TEXT("no widgets visited"), VisitCount, 0);
		});

		// NOTE: `# inherited` annotation is not emitted in this MVP — UWidget has no public
		// inherited flag in this engine version. Re-add this test when WidgetBlueprintGeneratedClass-
		// aware inherited detection lands.

		It("traverses named slot content with correct SlotName", [this]()
		{
			// UExpandableArea implements INamedSlotInterface with two slots: Header and Body.
			UExpandableArea* ExpandArea = TestBP->WidgetTree->ConstructWidget<UExpandableArea>(
				UExpandableArea::StaticClass(), TEXT("ExpandArea"));
			UTextBlock* HeaderText = TestBP->WidgetTree->ConstructWidget<UTextBlock>(
				UTextBlock::StaticClass(), TEXT("HeaderText"));

			UBorder* Root = Cast<UBorder>(TestBP->WidgetTree->RootWidget);
			TestNotNull(TEXT("Root is Border"), Root);
			Root->ClearChildren();
			Root->SetContent(ExpandArea);
			ExpandArea->SetContentForSlot(FName(TEXT("Header")), HeaderText);

			TArray<TPair<FName, FName>> Visited;  // {WidgetName, SlotName}
			FWidgetBlueprintOperationUtils::WalkWidgetTree(
				TestBP, nullptr, -1,
				[&](const UWidget* W, int32 /*Depth*/, FName SlotName)
				{
					Visited.Add({W->GetFName(), SlotName});
				});

			TestEqual(TEXT("widget count"), Visited.Num(), 3);
			TestEqual(TEXT("order[0]"), Visited[0].Key, FName(TEXT("Root")));
			TestEqual(TEXT("order[1]"), Visited[1].Key, FName(TEXT("ExpandArea")));
			TestEqual(TEXT("order[2]"), Visited[2].Key, FName(TEXT("HeaderText")));

			TestEqual(TEXT("HeaderText carries SlotName=Header"),
				Visited[2].Value, FName(TEXT("Header")));
			TestEqual(TEXT("ExpandArea has no slot name"),
				Visited[1].Value, NAME_None);
		});
	});

	Describe("FullMode", [this]()
	{
		BeforeEach([this]()
		{
			UMGToolSetTest::RegisterTestMountPoint();
			BuildFixture();
		});
		AfterEach([this]()
		{
			TeardownFixture();
			UMGToolSetTest::UnregisterTestMountPoint();
		});

		It("emits a non-CDO property value (TextBlock.Justification when set to non-default)", [this]()
		{
			UTextBlock* First = Cast<UTextBlock>(TestBP->WidgetTree->FindWidget(FName(TEXT("First"))));
			TestNotNull(TEXT("First found"), First);
			First->SetJustification(ETextJustify::Center);

			const FUMGWidgetDescriptionResult Result = UUMGToolSet::GetWidgetDescription(TestBP);

			TestTrue(TEXT("contains Justification field"),
				Result.Description.Contains(TEXT("Justification:")));

			TestEqual(TEXT("widget count"), Result.Widgets.Num(), 4);
		});

		It("emits slot:(...) block when slot has non-default property", [this]()
		{
			UTextBlock* First = Cast<UTextBlock>(TestBP->WidgetTree->FindWidget(FName(TEXT("First"))));
			TestNotNull(TEXT("First found"), First);
			UVerticalBoxSlot* Slot = Cast<UVerticalBoxSlot>(First->Slot);
			TestNotNull(TEXT("VBox slot"), Slot);
			Slot->SetPadding(FMargin(4, 0, 0, 0));

			const FUMGWidgetDescriptionResult Result = UUMGToolSet::GetWidgetDescription(TestBP);

			TestTrue(TEXT("contains slot block"), Result.Description.Contains(TEXT("slot:(")));
			TestTrue(TEXT("slot includes Padding"), Result.Description.Contains(TEXT("Padding:")));
		});

		It("skips properties matching CDO default", [this]()
		{
			const FUMGWidgetDescriptionResult Result = UUMGToolSet::GetWidgetDescription(TestBP);

			// Find the line containing "TextBlock First" -- format is now "[N] TextBlock First"
			const int32 FirstIdx = Result.Description.Find(TEXT("TextBlock First"));
			TestTrue(TEXT("found First line"), FirstIdx != INDEX_NONE);
			const int32 NextNewline = Result.Description.Find(
				TEXT("\n"), ESearchCase::CaseSensitive, ESearchDir::FromStart, FirstIdx);
			const FString FirstLine = Result.Description.Mid(FirstIdx, NextNewline - FirstIdx);
			TestFalse(TEXT("First line has no Justification field at default"),
				FirstLine.Contains(TEXT("Justification:")));
		});
	});

	Describe("GetWidgetTreeDepth", [this]()
	{
		BeforeEach([this]()
		{
			UMGToolSetTest::RegisterTestMountPoint();
			BuildFixture();
		});
		AfterEach([this]()
		{
			TeardownFixture();
			UMGToolSetTest::UnregisterTestMountPoint();
		});

		It("returns 2 for the standard fixture (Border > VBox > TextBlock)", [this]()
		{
			TestEqual(TEXT("depth"), UUMGToolSet::GetWidgetTreeDepth(TestBP), 2);
		});

		It("returns 0 for root-only tree", [this]()
		{
			FString OneName = FString::Printf(TEXT("WBP_OneWidget_%d"), TestCounter++);
			UWidgetBlueprint* OneWidget = UUMGToolSet::CreateWidgetBlueprint(
				UMGToolSetTest::TestMountPoint, OneName, UUserWidget::StaticClass());
			TestNotNull(TEXT("OneWidget BP created"), OneWidget);

			UTextBlock* Solo = OneWidget->WidgetTree->ConstructWidget<UTextBlock>(
				UTextBlock::StaticClass(), TEXT("Solo"));
			OneWidget->WidgetTree->RootWidget = Solo;

			TestEqual(TEXT("depth"), UUMGToolSet::GetWidgetTreeDepth(OneWidget), 0);
		});

		It("returns -1 on null WBP", [this]()
		{
			TestEqual(TEXT("depth"), UUMGToolSet::GetWidgetTreeDepth(nullptr), -1);
		});

		It("returns 0 on empty tree (WBP exists, no root widget)", [this]()
		{
			FString EmptyName = FString::Printf(TEXT("WBP_EmptyDepth_%d"), TestCounter++);
			UWidgetBlueprint* EmptyBP = UUMGToolSet::CreateWidgetBlueprint(
				UMGToolSetTest::TestMountPoint, EmptyName, UUserWidget::StaticClass());
			TestNotNull(TEXT("EmptyBP created"), EmptyBP);

			TestEqual(TEXT("depth"), UUMGToolSet::GetWidgetTreeDepth(EmptyBP), 0);
		});

		It("uses StartWidget as level 0 (depth 1 from Stack)", [this]()
		{
			UWidget* Stack = TestBP->WidgetTree->FindWidget(FName(TEXT("Stack")));
			TestNotNull(TEXT("Stack found"), Stack);
			TestEqual(TEXT("depth from Stack"),
				UUMGToolSet::GetWidgetTreeDepth(TestBP, Stack), 1);
		});
	});

	Describe("UFUNCTIONs", [this]()
	{
		BeforeEach([this]()
		{
			UMGToolSetTest::RegisterTestMountPoint();
			BuildFixture();
		});
		AfterEach([this]()
		{
			TeardownFixture();
			UMGToolSetTest::UnregisterTestMountPoint();
		});

		It("GetWidgetDescriptionFull returns non-empty for valid WBP", [this]()
		{
			const FUMGWidgetDescriptionResult Result = UUMGToolSet::GetWidgetDescription(TestBP);
			TestFalse(TEXT("non-empty description"), Result.Description.IsEmpty());
			TestTrue(TEXT("widgets populated"), Result.Widgets.Num() > 0);
		});

		It("GetWidgetTreeDepth returns 2 for fixture", [this]()
		{
			TestEqual(TEXT("depth"), UUMGToolSet::GetWidgetTreeDepth(TestBP), 2);
		});

		It("GetWidgetTreeDepth returns -1 on null WBP", [this]()
		{
			TestEqual(TEXT("depth"), UUMGToolSet::GetWidgetTreeDepth(nullptr), -1);
		});
	});
}

#endif // WITH_DEV_AUTOMATION_TESTS
