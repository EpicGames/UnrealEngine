// Copyright Epic Games, Inc. All Rights Reserved.

#include "UMGToolSet.h"
#include "Animation/WidgetAnimation.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Blueprint/UserWidget.h"
#include "Blueprint/WidgetTree.h"
#include "Blueprint/BlueprintExtension.h"
#include "Components/ContentWidget.h"
#include "Components/NamedSlotInterface.h"
#include "EdGraph/EdGraph.h"
#include "IMessageLogListing.h"
#include "K2Node_BaseMCDelegate.h"
#include "K2Node_CallFunction.h"
#include "K2Node_CallFunctionOnMember.h"
#include "K2Node_ComponentBoundEvent.h"
#include "K2Node_DynamicCast.h"
#include "K2Node_GetArrayItem.h"
#include "K2Node_Knot.h"
#include "K2Node_Select.h"
#include "K2Node_StructMemberGet.h"
#include "K2Node_Variable.h"
#include "K2Node_VariableGet.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Templates/WidgetTemplateClass.h"
#include "MovieScene.h"
#include "MovieSceneBinding.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "WidgetBlueprintOperationUtils.h"
#include "WidgetBlueprintExtension.h"
#include "UMGEditorProjectSettings.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealType.h"
#include "WidgetBlueprint.h"
#include "WidgetBlueprintEditorUtils.h"
#include "UIComponentWidgetBlueprintExtension.h"

DEFINE_LOG_CATEGORY_STATIC(LogUMGToolSet, Log, All);

// ============================================================================
// Internal helpers
// ============================================================================

namespace Private
{

/**
 * Returns an FSoftClassPath that round-trips through TSubclassOf.
 * Native classes: /Script/UMG.TextBlock (works as-is).
 * Blueprint classes: FSoftClassPath(GeneratedClass) produces a _C subobject path
 * that ToolsetRegistry's FSoftObjectPath::TryLoad can't resolve. Use the Blueprint
 * asset path instead — ToolsetRegistry loads the Blueprint and extracts the class.
 */
FSoftClassPath GetResolvableClassPath(UClass* Class)
{
	if (Class && Class->ClassGeneratedBy)
	{
		return FSoftClassPath(Class->ClassGeneratedBy->GetPathName());
	}
	return FSoftClassPath(Class);
}

/**
 * Returns true if WidgetName matches any UPROPERTY on the blueprint's parent class.
 * GUID registration is skipped for these names — the compiler maps them to the
 * existing C++ property (BindWidget, BindWidgetOptional, or any other UPROPERTY).
 */
bool IsParentClassPropertyName(UWidgetBlueprint* BP, FName WidgetName)
{
	if (!BP || WidgetName.IsNone())
	{
		return false;
	}

	UClass* ParentClass = BP->ParentClass;
	if (!ParentClass)
	{
		return false;
	}

	// Check if ANY property with this name exists on the parent class
	return FindFProperty<FProperty>(ParentClass, WidgetName) != nullptr;
}

/** Depth-first walk of widget tree, appending FUMGWidgetInfo entries. NamedSlotHostWidget is set when recursing into named slot content. */
void CollectWidgets(UWidget* Widget, UPanelWidget* Parent, UWidget* NamedSlotHostWidget, UWidgetBlueprint* BP, TArray<FUMGWidgetInfo>& OutWidgets)
{
	if (!Widget)
	{
		return;
	}

	FUMGWidgetInfo Info;
	Info.Widget = Widget;
	Info.Parent = Parent;
	Info.Slot = Widget->Slot;

	Info.NamedSlotHost = NamedSlotHostWidget;
	Info.WidgetClassPath = GetResolvableClassPath(Widget->GetClass());
	Info.WidgetName = Widget->GetFName();
	Info.bInherited = IsParentClassPropertyName(BP, Widget->GetFName());
	Info.bIsVariable = Widget->bIsVariable;
	OutWidgets.Add(MoveTemp(Info));

	if (UPanelWidget* Panel = Cast<UPanelWidget>(Widget))
	{
		for (int32 i = 0; i < Panel->GetChildrenCount(); i++)
		{
			CollectWidgets(Panel->GetChildAt(i), Panel, nullptr, BP, OutWidgets);
		}
	}

	// Recurse into named slot content on widget instances
	if (INamedSlotInterface* SlotHost = Cast<INamedSlotInterface>(Widget))
	{
		TArray<FName> SlotNames;
		SlotHost->GetSlotNames(SlotNames);
		for (const FName& SlotName : SlotNames)
		{
			if (UWidget* Content = SlotHost->GetContentForSlot(SlotName))
			{
				CollectWidgets(Content, nullptr, Widget, BP, OutWidgets);
			}
		}
	}
}

/** Build FUMGWidgetInfo for a single widget. Used by AddWidget, SetNamedSlotContent, RenameWidget. */
FUMGWidgetInfo MakeWidgetInfo(UWidget* Widget, UPanelWidget* Parent, UWidgetBlueprint* BP)
{
	FUMGWidgetInfo Info;
	if (!Widget)
	{
		return Info;
	}
	Info.Widget = Widget;
	Info.Parent = Parent;
	Info.Slot = Widget->Slot;

	Info.WidgetClassPath = GetResolvableClassPath(Widget->GetClass());
	Info.WidgetName = Widget->GetFName();
	Info.bIsVariable = Widget->bIsVariable;
	Info.bInherited = false;
	return Info;
}

/**
 * Builds a FUMGWidgetClassEntry from a widget class, sourcing description via FWidgetTemplateClass
 * so the toolset matches what the UMG editor's palette shows. Caller must ensure Class is non-null and
 * derives from UWidget.
 */
FUMGWidgetClassEntry MakeWidgetClassEntry(UClass* Class)
{
	FUMGWidgetClassEntry Entry;
	Entry.WidgetClass = Class;
	Entry.bIsPanel = Class->IsChildOf(UPanelWidget::StaticClass());

	FWidgetTemplateClass Template(Class);
	Entry.Category = Template.GetCategory();
	Entry.Description = Template.GetToolTipText();
	return Entry;
}

/** Returns true if the property should appear in the full widget description — edit-visible, non-transient. */
bool ShouldEmitProperty(const FProperty* InProperty)
{
	if (!InProperty ||
		!InProperty->HasAnyPropertyFlags(CPF_Edit) ||
		InProperty->HasAnyPropertyFlags(CPF_Transient) ||
		InProperty->HasAnyPropertyFlags(CPF_DisableEditOnTemplate))
	{
		return false;
	}

	return true;
}

/** Serializes a single property value to string; returns empty if export fails. */
FString ExportPropertyValue(const FProperty* InProperty, const UObject* Container, const UObject* CDO)
{
	if (!InProperty || !IsValid(Container))
		return FString();

	FString ValueStr;

	const bool bExported = InProperty->ExportText_InContainer(0, ValueStr, Container, CDO,
		const_cast<UObject*>(Container), PPF_None);

	if (!bExported)
	{
		UE_LOG(LogUMGToolSet, Verbose, TEXT("GetWidgetDescription: ExportText returned false for %s on %s"),
			*InProperty->GetName(), *Container->GetPathName());

		return FString();
	}
	return ValueStr;
}

/** Renders one widget line (class, name, ref, properties, slot) into the full description output. */
void RenderFullWidget(FStringBuilderBase& OutString, const UWidget* Widget, const int32 Depth, const FName SlotName, const int32 Index)
{
	if (!Widget)
		return;

	OutString.Append(FString::ChrN(Depth * 2, TEXT(' ')));

	OutString.Appendf(TEXT("[%d] %s %s"),
		Index,
		*Widget->GetClass()->GetName(),
		*Widget->GetFName().ToString());

	const UClass* WidgetClass = Widget->GetClass();
	const UObject* CDO = WidgetClass ? WidgetClass->GetDefaultObject(true) : nullptr;

	for (TFieldIterator<FProperty> It(WidgetClass); It; ++It)
	{
		const FProperty* Prop = *It;

		if (!ShouldEmitProperty(Prop))
			continue;

		const FString ValueStr = ExportPropertyValue(Prop, Widget, CDO);

		if (ValueStr.IsEmpty())
			continue;

		OutString.Appendf(TEXT("  %s:%s"), *Prop->GetName(), *ValueStr);
	}

	if (IsValid(Widget->Slot))
	{
		if (const UClass* SlotClass = Widget->Slot->GetClass())
		{
			const UObject* SlotCDO = SlotClass->GetDefaultObject(true);

			FString SlotInner;
			for (TFieldIterator<FProperty> It(SlotClass); It; ++It)
			{
				const FProperty* SlotProp = *It;

				if (!ShouldEmitProperty(SlotProp))
					continue;

				const FString ValueStr = ExportPropertyValue(SlotProp, Widget->Slot, SlotCDO);

				if (ValueStr.IsEmpty())
					continue;

				if (!SlotInner.IsEmpty())
					SlotInner.Append(TEXT(", "));

				SlotInner.Appendf(TEXT("%s:%s"), *SlotProp->GetName(), *ValueStr);
			}

			if (!SlotInner.IsEmpty())
				OutString.Appendf(TEXT("  slot:(%s)"), *SlotInner);
		}
	}

	if (SlotName != NAME_None)
		OutString.Appendf(TEXT("  # named-slot:%s"), *SlotName.ToString());

	OutString.AppendChar(TEXT('\n'));
}

/**
 * Walks the blueprint-visible members declared on OldClass above UWidget/UUserWidget and
 * collects those that have no compatible counterpart on NewClass, with a per-member reason.
 */
static void CollectUnmatchedMembers(UClass* OldClass, UClass* NewClass, TArray<FWidgetUnmatchedMember>& OutUnmatchedProperties, TArray<FWidgetUnmatchedMember>& OutUnmatchedFunctions)
{
	if (!OldClass || !NewClass)
	{
		return;
	}

	for (TFieldIterator<FProperty> PropertyIt(OldClass); PropertyIt && PropertyIt.GetStruct() != UUserWidget::StaticClass() && PropertyIt.GetStruct() != UWidget::StaticClass(); ++PropertyIt)
	{
		FProperty* Property = *PropertyIt;
		if (!Property || !Property->HasAnyPropertyFlags(CPF_BlueprintVisible | CPF_BlueprintAssignable))
		{
			continue;
		}
		FText Reason;
		if (!FWidgetBlueprintOperationUtils::DoesPropertyExistInClass(NewClass, Property, Reason))
		{
			OutUnmatchedProperties.Add(FWidgetUnmatchedMember{ Property->GetFName(), MoveTemp(Reason) });
		}
	}

	for (TFieldIterator<UFunction> FuncIt(OldClass); FuncIt && FuncIt.GetStruct() != UUserWidget::StaticClass() && FuncIt.GetStruct() != UWidget::StaticClass(); ++FuncIt)
	{
		UFunction* Function = *FuncIt;
		if (!Function || !Function->HasAnyFunctionFlags(FUNC_BlueprintCallable | FUNC_BlueprintEvent))
		{
			continue;
		}
		FText Reason;
		if (!FWidgetBlueprintOperationUtils::DoesFunctionExistInClass(NewClass, Function, Reason))
		{
			OutUnmatchedFunctions.Add(FWidgetUnmatchedMember{ Function->GetFName(), MoveTemp(Reason) });
		}
	}
}

/**
 * Walks the blueprint and records every member of OldClass referenced through the named widget
 * variable: graph nodes (both pin-flow and name-keyed), legacy delegate bindings, animation
 * property tracks, and extension-defined references via UWidgetBlueprintExtension.
 */
static void CollectReferencedMembers(UWidgetBlueprint* BP, UClass* OldClass, FName VariableName, TSet<FName>& OutReferencedProperties, TSet<FName>& OutReferencedFunctions)
{
	if (!BP || !OldClass || VariableName.IsNone())
	{
		return;
	}

	TArray<UEdGraph*> AllGraphs;
	BP->GetAllGraphs(AllGraphs);

	// Get any graph not included in GetAllGraphs
	TSet<UEdGraph*> SeenGraphs(AllGraphs);
	ForEachObjectWithOuter(BP, [&AllGraphs, &SeenGraphs](UObject* Inner)
		{
			if (UEdGraph* Graph = Cast<UEdGraph>(Inner))
			{
				bool bAlreadyIn = false;
				SeenGraphs.Add(Graph, &bAlreadyIn);
				if (!bAlreadyIn)
				{
					AllGraphs.Add(Graph);
					Graph->GetAllChildrenGraphs(AllGraphs);
				}
			}
		}, EGetObjectsFlags::IncludeNestedObjects);

	// Phase 1: BFS forward from the variable's Get nodes, following the widget value through
	// reroutes / casts / array gets / selects.
	TArray<UEdGraphPin*> Worklist;
	for (UEdGraph* Graph : AllGraphs)
	{
		for (UEdGraphNode* Node : Graph->Nodes)
		{
			if (UK2Node_VariableGet* VarGet = Cast<UK2Node_VariableGet>(Node))
			{
				if (VarGet->VariableReference.IsSelfContext() && VarGet->GetVarName() == VariableName)
				{
					if (UEdGraphPin* ValuePin = VarGet->GetValuePin())
					{
						Worklist.Append(ValuePin->LinkedTo);
					}
				}
			}
		}
	}

	TSet<UEdGraphNode*> VisitedNodes;
	for (int32 WorkIdx = 0; WorkIdx < Worklist.Num(); ++WorkIdx)
	{
		UEdGraphNode* Node = Worklist[WorkIdx]->GetOwningNode();
		bool bAlreadyVisited = false;
		VisitedNodes.Add(Node, &bAlreadyVisited);
		if (bAlreadyVisited)
		{
			continue;
		}

		if (UK2Node_Knot* Knot = Cast<UK2Node_Knot>(Node))
		{
			if (UEdGraphPin* Out = Knot->GetOutputPin())
			{
				Worklist.Append(Out->LinkedTo);
			}
		}
		else if (UK2Node_DynamicCast* CastNode = Cast<UK2Node_DynamicCast>(Node))
		{
			if (UEdGraphPin* CastResult = CastNode->GetCastResultPin())
			{
				Worklist.Append(CastResult->LinkedTo);
			}
		}
		else if (UK2Node_GetArrayItem* GetItem = Cast<UK2Node_GetArrayItem>(Node))
		{
			if (UEdGraphPin* Result = GetItem->GetResultPin())
			{
				Worklist.Append(Result->LinkedTo);
			}
		}
		else if (UK2Node_Select* SelectNode = Cast<UK2Node_Select>(Node))
		{
			if (UEdGraphPin* Ret = SelectNode->GetReturnValuePin())
			{
				Worklist.Append(Ret->LinkedTo);
			}
		}
		else if (Cast<UK2Node_StructMemberGet>(Node) != nullptr)
		{
			for (UEdGraphPin* Pin : Node->Pins)
			{
				if (Pin->Direction == EGPD_Output && Pin->PinType.PinSubCategoryObject.Get() == OldClass)
				{
					Worklist.Append(Pin->LinkedTo);
				}
			}
		}
		else if (UK2Node_CallFunction* CallNode = Cast<UK2Node_CallFunction>(Node))
		{
			if (CallNode->FunctionReference.GetMemberParentClass() == OldClass)
			{
				OutReferencedFunctions.Add(CallNode->FunctionReference.GetMemberName());
			}
		}
		else if (UK2Node_BaseMCDelegate* DelegateNode = Cast<UK2Node_BaseMCDelegate>(Node))
		{
			if (DelegateNode->DelegateReference.GetMemberParentClass() == OldClass)
			{
				OutReferencedFunctions.Add(DelegateNode->DelegateReference.GetMemberName());
			}
		}
		else if (UK2Node_Variable* VarNode = Cast<UK2Node_Variable>(Node))
		{
			if (!VarNode->VariableReference.IsSelfContext() &&
				!VarNode->VariableReference.IsLocalScope() &&
				VarNode->VariableReference.GetMemberParentClass() == OldClass)
			{
				OutReferencedProperties.Add(VarNode->VariableReference.GetMemberName());
			}
		}
	}

	// Phase 2: name-keyed references that don't flow through pins.
	for (UEdGraph* Graph : AllGraphs)
	{
		for (UEdGraphNode* Node : Graph->Nodes)
		{
			if (UK2Node_ComponentBoundEvent* EventNode = Cast<UK2Node_ComponentBoundEvent>(Node))
			{
				if (EventNode->ComponentPropertyName == VariableName && EventNode->DelegateOwnerClass == OldClass)
				{
					OutReferencedFunctions.Add(EventNode->DelegatePropertyName);
				}
			}
			else if (UK2Node_CallFunctionOnMember* MemberCallNode = Cast<UK2Node_CallFunctionOnMember>(Node))
			{
				if (MemberCallNode->MemberVariableToCallOn.GetMemberName() == VariableName &&
					MemberCallNode->FunctionReference.GetMemberParentClass() == OldClass)
				{
					OutReferencedFunctions.Add(MemberCallNode->FunctionReference.GetMemberName());
				}
			}
		}
	}

	// Phase 3: legacy property-delegate bindings.
	const FString VariableNameString = VariableName.ToString();
	for (const FDelegateEditorBinding& Binding : BP->Bindings)
	{
		if (Binding.ObjectName == VariableNameString && !Binding.PropertyName.IsNone())
		{
			OutReferencedProperties.Add(Binding.PropertyName);
		}
	}

	// Phase 4: animation bindings. Each binding is keyed by widget name; each tracked property
	// inside the bound possessable identifies a property on the widget.
	for (UWidgetAnimation* Animation : BP->Animations)
	{
		if (!Animation || !Animation->MovieScene)
		{
			continue;
		}
		for (const FWidgetAnimationBinding& AnimBinding : Animation->AnimationBindings)
		{
			if (AnimBinding.WidgetName != VariableName)
			{
				continue;
			}
			const FMovieSceneBinding* SceneBinding = Animation->MovieScene->FindBinding(AnimBinding.AnimationGuid);
			if (!SceneBinding)
			{
				continue;
			}
			for (UMovieSceneTrack* Track : SceneBinding->GetTracks())
			{
				if (UMovieScenePropertyTrack* PropertyTrack = Cast<UMovieScenePropertyTrack>(Track))
				{
					const FName AnimatedProperty = PropertyTrack->GetPropertyName();
					if (!AnimatedProperty.IsNone())
					{
						OutReferencedProperties.Add(AnimatedProperty);
					}
				}
			}
		}
	}

	// Phase 5: extension-defined references. Each UWidgetBlueprintExtension reports its own
	// widget-member references via GatherWidgetMemberReferences
	for (const TObjectPtr<UBlueprintExtension>& Extension : BP->GetExtensions())
	{
		if (const UWidgetBlueprintExtension* WidgetExt = Cast<UWidgetBlueprintExtension>(Extension))
		{
			WidgetExt->GatherWidgetMemberReferences(VariableName, OldClass, OutReferencedProperties, OutReferencedFunctions);
		}
	}
}
} // Private namespace


// ============================================================================
// Creation
// ============================================================================

UWidgetBlueprint* UUMGToolSet::CreateWidgetBlueprint(const FString& FolderPath, const FString& AssetName, TSubclassOf<UUserWidget> ParentClass)
{
	if (FolderPath.IsEmpty() || AssetName.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("CreateWidgetBlueprint: FolderPath and AssetName are required."));
		return nullptr;
	}

	if (!ParentClass)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("CreateWidgetBlueprint: ParentClass is required."));
		return nullptr;
	}

	FString NormalizedPath = FolderPath;
	if (!NormalizedPath.StartsWith(TEXT("/")))
	{
		NormalizedPath = TEXT("/") + NormalizedPath;
	}
	NormalizedPath.RemoveFromEnd(TEXT("/"));

	const FString PackagePath = NormalizedPath / AssetName;

	// Do not clobber an existing asset.
	if (FindPackage(nullptr, *PackagePath))
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("CreateWidgetBlueprint: Failed to create blueprint '%s' at '%s'. The path already exists."),
			*AssetName, *FolderPath));
		return nullptr;
	}

	UPackage* Package = CreatePackage(*PackagePath);
	if (!Package)
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("CreateWidgetBlueprint: Failed to create package for blueprint '%s' at '%s'."),
			*AssetName, *FolderPath));
		return nullptr;
	}

	UClass* RootWidgetClass = GetDefault<UUMGEditorProjectSettings>()->DefaultRootWidget;

	UWidgetBlueprint* BP = FWidgetBlueprintOperationUtils::CreateWidgetBlueprint(Package, FName(*AssetName), BPTYPE_Normal, ParentClass, RootWidgetClass, NAME_None, /*bRegisterAndCompile=*/true);

	if (!BP)
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("CreateWidgetBlueprint: Failed to create blueprint '%s' at '%s'."),
			*AssetName, *FolderPath));
		return nullptr;
	}

	return BP;
}

bool UUMGToolSet::BindToEventProperty(UWidgetBlueprint* WidgetBlueprint, const FName EventName, FName PropertyName, UClass* PropertyClass)
{
	FText ErrorMessage;
	const bool bResult = FWidgetBlueprintOperationUtils::BindToEventProperty(WidgetBlueprint, EventName, PropertyName, PropertyClass, false, ErrorMessage);

	if (!bResult)
	{
		UKismetSystemLibrary::RaiseScriptError(ErrorMessage.ToString());
	}

	return bResult;
}

TArray<FUMGWidgetInfo> UUMGToolSet::WrapWidgets(UWidgetBlueprint* WidgetBlueprint, TArray<UWidget*> Widgets, TSubclassOf<UPanelWidget> WrapperClass)
{
	TArray<FUMGWidgetInfo> Result;

	if (!WidgetBlueprint || !WrapperClass.Get() || Widgets.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("WrapWidgets: WidgetBlueprint, WrapperClass, and at least one Widget are required."));
		return Result;
	}

	TArray<UWidget*> NewWrappers = FWidgetBlueprintOperationUtils::WrapWidgets(WidgetBlueprint, Widgets, WrapperClass.Get());

	for (UWidget* Wrapper : NewWrappers)
	{
		UPanelWidget* ParentOfWrapper = Wrapper->Slot ? Cast<UPanelWidget>(Wrapper->Slot->Parent) : nullptr;
		Result.Add(Private::MakeWidgetInfo(Wrapper, ParentOfWrapper, WidgetBlueprint));
	}

	return Result;
}

FUMGWidgetInfo UUMGToolSet::AddWidget(UWidgetBlueprint* WidgetBlueprint, TSubclassOf<UWidget> WidgetClass, const FString& WidgetDisplayName, UWidget* ParentWidget, int32 ChildIndex)
{
	FUMGWidgetInfo EmptyInfo;

	if (!WidgetBlueprint || !WidgetClass.Get())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("AddWidget: WidgetBlueprint and WidgetClass is required."));
		return EmptyInfo;
	}

	FText OutErrorMessage;

	FAssetData Asset = FAssetData(WidgetClass.Get());
	UWidget* NewWidget = FWidgetBlueprintOperationUtils::CreateWidgetFromAsset(WidgetBlueprint, Asset, WidgetBlueprint->WidgetTree, OutErrorMessage);

	if (!NewWidget)
	{
		if (!OutErrorMessage.IsEmpty())
		{
			UKismetSystemLibrary::RaiseScriptError(FString::Printf(
				TEXT("AddWidget: '%s'."), *OutErrorMessage.ToString()));
		}
		else
		{
			UKismetSystemLibrary::RaiseScriptError(TEXT("AddWidget: Failed to construct widget."));
		}

		return EmptyInfo;
	}

	OutErrorMessage = FText::GetEmpty();
	if (!FWidgetBlueprintOperationUtils::VerifyWidgetRename(WidgetBlueprint, NewWidget, FText::FromString(WidgetDisplayName), OutErrorMessage))
	{
		FWidgetBlueprintOperationUtils::RemoveTransientWidgetFromTree(WidgetBlueprint, NewWidget);

		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("AddWidget: '%s'."), *OutErrorMessage.ToString()));
		return EmptyInfo;
	}

	if (!FWidgetBlueprintOperationUtils::AddWidget(WidgetBlueprint, NewWidget, ParentWidget, ChildIndex, OutErrorMessage))
	{
		FWidgetBlueprintOperationUtils::RemoveTransientWidgetFromTree(WidgetBlueprint, NewWidget);

		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("AddWidget: '%s'."), *OutErrorMessage.ToString()));
		return EmptyInfo;
	}

	if (!FWidgetBlueprintOperationUtils::RenameWidget(WidgetBlueprint, NewWidget, WidgetDisplayName))
	{
		// If rename failed, remove the widget.
		OutErrorMessage = FText::GetEmpty();
		FWidgetBlueprintOperationUtils::RemoveWidget(WidgetBlueprint, NewWidget, OutErrorMessage);

			UKismetSystemLibrary::RaiseScriptError(FString::Printf(
				TEXT("AddWidget: Failed to name widget to '%s'. Deleting the widget."), *WidgetDisplayName));
		return EmptyInfo;
	}

	UPanelWidget* ActualParent = NewWidget->Slot ? Cast<UPanelWidget>(NewWidget->Slot->Parent) : nullptr;
	return Private::MakeWidgetInfo(NewWidget, ActualParent, WidgetBlueprint);
}

FUMGWidgetInfo UUMGToolSet::SetNamedSlotContent(UWidgetBlueprint* WidgetBlueprint, UWidget* HostWidget, FName SlotName, TSubclassOf<UWidget> WidgetClass, FName WidgetName)
{
	FUMGWidgetInfo EmptyInfo;

	if (!WidgetBlueprint)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("SetNamedSlotContent: WidgetBlueprint is required."));
		return EmptyInfo;
	}

	if (SlotName.IsNone() || !WidgetClass || WidgetName.IsNone())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("SetNamedSlotContent: SlotName, WidgetClass, and WidgetName are required."));
		return EmptyInfo;
	}

	if (WidgetClass->HasAnyClassFlags(CLASS_Abstract))
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("SetNamedSlotContent: '%s' is abstract and cannot be instantiated."), *WidgetClass->GetName()));
		return EmptyInfo;
	}

	if (Private::IsParentClassPropertyName(WidgetBlueprint, WidgetName))
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("SetNamedSlotContent: '%s' conflicts with a C++ property on the parent class."), *WidgetName.ToString()));
		return EmptyInfo;
	}

	UWidgetTree* WidgetTree = WidgetBlueprint->WidgetTree;
	if (!WidgetTree)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("SetNamedSlotContent: WidgetBlueprint has no WidgetTree."));
		return EmptyInfo;
	}

	INamedSlotInterface* SlotHost = nullptr;
	if (!HostWidget)
	{
		SlotHost = WidgetTree;
	}
	else
	{
		SlotHost = Cast<INamedSlotInterface>(HostWidget);
		if (!SlotHost)
		{
			UKismetSystemLibrary::RaiseScriptError(FString::Printf(
				TEXT("SetNamedSlotContent: HostWidget '%s' (%s) does not implement INamedSlotInterface."),
				*HostWidget->GetName(), *HostWidget->GetClass()->GetName()));
			return EmptyInfo;
		}
	}

	// If widget already exists in the tree, reparent it into the named slot
	UWidget* ExistingWidget = WidgetTree->FindWidget(WidgetName);
	if (ExistingWidget)
	{
		WidgetBlueprint->Modify();

		// Unparent from current panel if needed
		if (ExistingWidget->Slot && ExistingWidget->Slot->Parent)
		{
			ExistingWidget->Slot->Parent->RemoveChild(ExistingWidget);
		}

		SlotHost->SetContentForSlot(SlotName, ExistingWidget);
		WidgetBlueprint->MarkPackageDirty();

		return Private::MakeWidgetInfo(ExistingWidget, nullptr, WidgetBlueprint);
	}

	WidgetBlueprint->Modify();

	UWidget* NewWidget = WidgetTree->ConstructWidget<UWidget>(WidgetClass, WidgetName);
	if (!NewWidget)
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("SetNamedSlotContent: Failed to construct widget '%s'."), *WidgetName.ToString()));
		return EmptyInfo;
	}

	if (!WidgetBlueprint->WidgetVariableNameToGuidMap.Contains(NewWidget->GetFName()))
	{
		WidgetBlueprint->OnVariableAdded(NewWidget->GetFName());
	}

	SlotHost->SetContentForSlot(SlotName, NewWidget);
	WidgetBlueprint->MarkPackageDirty();

	return Private::MakeWidgetInfo(NewWidget, nullptr, WidgetBlueprint);

}

// ============================================================================
// Query
// ============================================================================

FUMGWidgetTreeInfo UUMGToolSet::GetWidgets(UWidgetBlueprint* WidgetBlueprint)
{
	FUMGWidgetTreeInfo Result;

	if (!WidgetBlueprint)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("GetWidgets: WidgetBlueprint is required."));
		return Result;
	}

	if (!WidgetBlueprint->WidgetTree)
	{
		return Result;
	}

	UWidgetTree* WidgetTree = WidgetBlueprint->WidgetTree;

	// Populate blueprint info
	Result.Info.ParentClass = TSubclassOf<UUserWidget>(WidgetBlueprint->ParentClass);

	if (WidgetTree->RootWidget)
	{
		Result.Info.RootWidgetClass = WidgetTree->RootWidget->GetClass();
	}

	// Collect tree hierarchy
	if (WidgetTree->RootWidget)
	{
		Private::CollectWidgets(WidgetTree->RootWidget, nullptr, nullptr, WidgetBlueprint, Result.Widgets);
	}

	// Collect inherited named slot content
	for (const auto& Binding : WidgetTree->NamedSlotBindings)
	{
		if (Binding.Value)
		{
			Private::CollectWidgets(Binding.Value, nullptr, nullptr, WidgetBlueprint, Result.Widgets);
		}
	}

	// Collect any remaining widgets not reached by the tree walk (e.g. named slot
	// content on inherited C++ host widgets outside the panel hierarchy).
	TSet<UWidget*> Visited;
	for (const FUMGWidgetInfo& Info : Result.Widgets)
	{
		Visited.Add(Info.Widget.Get());
	}
	WidgetTree->ForEachWidget([&](UWidget* Widget)
	{
		if (Widget && !Visited.Contains(Widget))
		{
			FUMGWidgetInfo Info;
			Info.Widget = Widget;
			Info.Parent = Widget->Slot ? Cast<UPanelWidget>(Widget->Slot->Parent) : nullptr;
			Info.Slot = Widget->Slot;
			Info.WidgetClassPath = Private::GetResolvableClassPath(Widget->GetClass());
			Info.WidgetName = Widget->GetFName();
			Info.bInherited = Private::IsParentClassPropertyName(WidgetBlueprint, Widget->GetFName());
			Info.bIsVariable = Widget->bIsVariable;
			Result.Widgets.Add(MoveTemp(Info));
		}
	});

	// Add placeholder entries for unbound BindWidget properties from parent class.
	// These widgets are defined in C++ but don't exist in the WidgetTree yet.
	// Widget pointer is null — callers see the expected name and class.
	// Only add when the tree has content (skip empty/uninitialized blueprints).
	TSet<FName> WidgetNamesInTree;
	for (const FUMGWidgetInfo& Info : Result.Widgets)
	{
		WidgetNamesInTree.Add(Info.WidgetName);
	}
	if (WidgetTree->RootWidget && WidgetBlueprint->ParentClass)
	{
		UClass* ParentClass = WidgetBlueprint->ParentClass;
		for (TFieldIterator<FObjectProperty> It(ParentClass, EFieldIteratorFlags::IncludeSuper); It; ++It)
		{
			// Only include properties tagged with BindWidget or BindWidgetOptional
			if (It->PropertyClass && It->PropertyClass->IsChildOf(UWidget::StaticClass())
				&& (It->HasMetaData(TEXT("BindWidget")) || It->HasMetaData(TEXT("BindWidgetOptional"))))
			{
				FName PropName = It->GetFName();
				if (!WidgetNamesInTree.Contains(PropName))
				{
					FUMGWidgetInfo Placeholder;
					// Widget is null — not in the tree
					Placeholder.WidgetClassPath = FSoftClassPath(It->PropertyClass);
					Placeholder.WidgetName = PropName;
					Placeholder.bInherited = true;
					Result.Widgets.Add(MoveTemp(Placeholder));
				}
			}
		}
	}

	// Populate UIComponents for each widget from the blueprint extension
	if (UUIComponentWidgetBlueprintExtension* Ext = UWidgetBlueprintExtension::GetExtension<UUIComponentWidgetBlueprintExtension>(WidgetBlueprint))
	{
		for (FUMGWidgetInfo& WidgetInfo : Result.Widgets)
		{
			if (!WidgetInfo.Widget)
			{
				continue;
			}
			for (UUIComponent* Comp : Ext->GetComponentsFor(WidgetInfo.Widget.Get()))
			{
				if (Comp)
				{
					FUMGUIComponentInfo CompInfo;
					CompInfo.Component = Comp;
					CompInfo.ComponentClassPath = Private::GetResolvableClassPath(Comp->GetClass());
					WidgetInfo.UIComponents.Add(MoveTemp(CompInfo));
				}
			}
		}
	}

	// Populate counts from collected widgets
	Result.Info.WidgetCount = Result.Widgets.Num();
	for (const FUMGWidgetInfo& WidgetInfo : Result.Widgets)
	{
		if (WidgetInfo.bInherited)
		{
			Result.Info.InheritedWidgetCount++;
		}
	}

	// Count named slot bindings
	Result.Info.NamedSlotCount = WidgetTree->NamedSlotBindings.Num();
	WidgetTree->ForEachWidget([&](UWidget* Widget)
	{
		if (INamedSlotInterface* SlotHost = Cast<INamedSlotInterface>(Widget))
		{
			TArray<FName> SlotNames;
			SlotHost->GetSlotNames(SlotNames);
			for (const FName& SlotName : SlotNames)
			{
				if (SlotHost->GetContentForSlot(SlotName))
				{
					Result.Info.NamedSlotCount++;
				}
			}
		}
	});

	return Result;
}

TArray<FUMGNamedSlotEntry> UUMGToolSet::GetNamedSlots(UWidgetBlueprint* WidgetBlueprint)
{
	TArray<FUMGNamedSlotEntry> Result;

	if (!WidgetBlueprint)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("GetNamedSlots: WidgetBlueprint is required."));
		return Result;
	}

	if (!WidgetBlueprint->WidgetTree)
	{
		return Result;
	}

	UWidgetTree* WidgetTree = WidgetBlueprint->WidgetTree;

	// Inherited named slots (from parent class)
	for (const auto& Binding : WidgetTree->NamedSlotBindings)
	{
		if (Binding.Value)
		{
			FUMGNamedSlotEntry Entry;
			Entry.SlotName = Binding.Key;
			// HostWidget nullptr = inherited from parent class
			Entry.ContentWidget = Binding.Value;
			Result.Add(MoveTemp(Entry));
		}
	}

	// Named slots on widget instances in the tree
	WidgetTree->ForEachWidget([&](UWidget* Widget)
	{
		if (INamedSlotInterface* SlotHost = Cast<INamedSlotInterface>(Widget))
		{
			TArray<FName> SlotNames;
			SlotHost->GetSlotNames(SlotNames);
			for (const FName& SlotName : SlotNames)
			{
				if (UWidget* Content = SlotHost->GetContentForSlot(SlotName))
				{
					FUMGNamedSlotEntry Entry;
					Entry.SlotName = SlotName;
					Entry.HostWidget = Widget;
					Entry.ContentWidget = Content;
					Result.Add(MoveTemp(Entry));
				}
			}
		}
	});

	return Result;
}

FWidgetReplacementReport UUMGToolSet::ReplaceWidgetWithTemplate(UWidgetBlueprint* WidgetBlueprint, UWidget* WidgetToReplace, TSubclassOf<UWidget> TemplateClass)
{
	FWidgetReplacementReport Report;

	if (!WidgetBlueprint || !WidgetToReplace || !TemplateClass)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("ReplaceWidgetWithTemplate: WidgetBlueprint, WidgetToReplace, and TemplateClass must all be valid."));
		return Report;
	}

	// Enumerate the public members on the old class with no compatible counterpart on the new class, and intersect with what's actually referenced.
	UClass* OldClass = WidgetToReplace->GetClass();
	const FName OriginalName = WidgetToReplace->GetFName();

	Private::CollectUnmatchedMembers(OldClass, TemplateClass, Report.UnmatchedProperties, Report.UnmatchedFunctions);

	TSet<FName> ReferencedProperties;
	TSet<FName> ReferencedFunctions;
	Private::CollectReferencedMembers(WidgetBlueprint, OldClass, OriginalName, ReferencedProperties, ReferencedFunctions);

	for (const FWidgetUnmatchedMember& UnmatchedProperty : Report.UnmatchedProperties)
	{
		if (ReferencedProperties.Contains(UnmatchedProperty.Name))
		{
			Report.UnmatchedReferencedProperties.Add(UnmatchedProperty);
		}
	}
	for (const FWidgetUnmatchedMember& UnmatchedFunction : Report.UnmatchedFunctions)
	{
		if (ReferencedFunctions.Contains(UnmatchedFunction.Name))
		{
			Report.UnmatchedReferencedFunctions.Add(UnmatchedFunction);
		}
	}

	// This is not a hard fail so that we can support replacement-then-remap. 
	// In some cases, we might want to map a function/property in the old widget class to another function/property with a different name in the new widget class.
	if (Report.UnmatchedReferencedProperties.Num() > 0 || Report.UnmatchedReferencedFunctions.Num() > 0)
	{
		TArray<FString> Lines;
		Lines.Reserve(Report.UnmatchedReferencedProperties.Num() + Report.UnmatchedReferencedFunctions.Num() + 2);

		if (Report.UnmatchedReferencedProperties.Num() > 0)
		{
			Lines.Add(FString::Printf(TEXT("Properties (%d):"), Report.UnmatchedReferencedProperties.Num()));
			for (const FWidgetUnmatchedMember& Member : Report.UnmatchedReferencedProperties)
			{
				Lines.Add(FString::Printf(TEXT("  - %s: %s"), *Member.Name.ToString(), *Member.Reason.ToString()));
			}
		}
		if (Report.UnmatchedReferencedFunctions.Num() > 0)
		{
			Lines.Add(FString::Printf(TEXT("Functions (%d):"), Report.UnmatchedReferencedFunctions.Num()));
			for (const FWidgetUnmatchedMember& Member : Report.UnmatchedReferencedFunctions)
			{
				Lines.Add(FString::Printf(TEXT("  - %s: %s"), *Member.Name.ToString(), *Member.Reason.ToString()));
			}
		}

		Report.MissingReferencesWarning = FText::Format(
			NSLOCTEXT("UMGToolSet", "ReplaceWidgetWithTemplate_UnmatchedReferencedMembers",
				"Warning on replacing widget '{0}' with class '{1}': {2} member(s) referenced by the blueprint have no compatible counterpart on the new class. Remove or update those references, otherwise the widget blueprint won't compile.\n\n{3}"),
			FText::FromName(OriginalName),
			FText::FromString(TemplateClass->GetName()),
			FText::AsNumber(Report.UnmatchedReferencedProperties.Num() + Report.UnmatchedReferencedFunctions.Num()),
			FText::FromString(FString::Join(Lines, TEXT("\n"))));

		UE_LOG(LogUMGToolSet, Warning, TEXT("%s"), *Report.MissingReferencesWarning.ToString());
	}

	FText ReplaceError;
	if (!FWidgetBlueprintOperationUtils::ReplaceWidgetWithTemplate(WidgetBlueprint, WidgetToReplace, TemplateClass, ReplaceError))
	{
		UKismetSystemLibrary::RaiseScriptError(ReplaceError.ToString());
		return Report;
	}

	Report.bSuccess = true;
	return Report;
}

bool UUMGToolSet::ReplaceWidgetWithNamedSlot(UWidgetBlueprint* WidgetBlueprint, UWidget* WidgetToReplace, FName NamedSlot)
{
	FText OutErrorMessage;
	if (!FWidgetBlueprintOperationUtils::ReplaceWidgetWithNamedSlot(WidgetBlueprint, WidgetToReplace, NamedSlot, FWidgetBlueprintEditorUtils::EDeleteWidgetWarningType::DeleteSilently, OutErrorMessage))
	{
		UKismetSystemLibrary::RaiseScriptError(OutErrorMessage.ToString());
		return false;
	}
	return true;
}

bool UUMGToolSet::ReplaceWidgetWithChild(UWidgetBlueprint* WidgetBlueprint, UWidget* WidgetToReplace)
{
	FText OutErrorMessage;
	if (!FWidgetBlueprintOperationUtils::ReplaceWidgetWithChild(WidgetBlueprint, WidgetToReplace, OutErrorMessage))
	{
		UKismetSystemLibrary::RaiseScriptError(OutErrorMessage.ToString());
		return false;
	}
	return true;
}

TArray<FSoftObjectPath> UUMGToolSet::ListWidgetBlueprints(const FString& FolderPath)
{
	TArray<FSoftObjectPath> Result;

	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();

	FARFilter Filter;
	Filter.ClassPaths.Add(UWidgetBlueprint::StaticClass()->GetClassPathName());
	Filter.bRecursivePaths = true;
	Filter.bRecursiveClasses = true;

	FString NormalizedPath = FolderPath;
	if (!NormalizedPath.StartsWith(TEXT("/")))
	{
		NormalizedPath = TEXT("/") + NormalizedPath;
	}
	if (!NormalizedPath.EndsWith(TEXT("/")))
	{
		NormalizedPath += TEXT("/");
	}
	Filter.PackagePaths.Add(FName(*NormalizedPath));

	TArray<FAssetData> AssetDataList;
	AssetRegistry.GetAssets(Filter, AssetDataList);

	for (const FAssetData& AssetData : AssetDataList)
	{
		Result.Add(AssetData.GetSoftObjectPath());
	}

	return Result;
}

TArray<FUMGWidgetClassEntry> UUMGToolSet::ListWidgetClasses(const FString& Filter)
{
	TArray<FUMGWidgetClassEntry> Result;

	for (TObjectIterator<UClass> It; It; ++It)
	{
		UClass* Class = *It;
		if (!Class->IsChildOf(UWidget::StaticClass()))
		{
			continue;
		}
		if (Class->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated))
		{
			continue;
		}

		if (!Filter.IsEmpty() && !Class->GetName().Contains(Filter))
		{
			continue;
		}

		Result.Add(Private::MakeWidgetClassEntry(Class));
	}

	return Result;
}

FUMGWidgetClassEntry UUMGToolSet::GetWidgetClassInfo(TSubclassOf<UWidget> WidgetClass)
{
	if (!WidgetClass)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("GetWidgetClassInfo: WidgetClass is required."));
		return FUMGWidgetClassEntry();
	}

	return Private::MakeWidgetClassEntry(WidgetClass.Get());
}

// ============================================================================
// Modification
// ============================================================================

FUMGWidgetInfo UUMGToolSet::MoveWidget(UWidgetBlueprint* WidgetBlueprint, UWidget* Widget, UPanelWidget* NewParent, int32 ChildIndex)
{
	FUMGWidgetInfo EmptyInfo;
	if (!WidgetBlueprint || !Widget || !NewParent)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("MoveWidget: WidgetBlueprint and Widget and NewParent are required."));
		return EmptyInfo;
	}

	FText OutErrorMessage;
	if (!FWidgetBlueprintOperationUtils::MoveWidget(WidgetBlueprint, Widget, NewParent, ChildIndex, OutErrorMessage))
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("MoveWidget: NewParent '%s' (%s) rejected widget '%s' at index %d. The provided error: '%s'"),
			*NewParent->GetName(), *NewParent->GetClass()->GetName(), *Widget->GetName(), ChildIndex, *OutErrorMessage.ToString()));
		return EmptyInfo;
	}

	return Private::MakeWidgetInfo(Widget, NewParent, WidgetBlueprint);
}

bool UUMGToolSet::RemoveWidget(UWidgetBlueprint* WidgetBlueprint, UWidget* Widget)
{
	FText OutErrorMessage;
	if (!FWidgetBlueprintOperationUtils::RemoveWidget(WidgetBlueprint, Widget, OutErrorMessage))
	{
		UKismetSystemLibrary::RaiseScriptError(OutErrorMessage.ToString());
		return false;
	}
	
	return true;
}

FUMGWidgetInfo UUMGToolSet::RenameWidget(UWidgetBlueprint* WidgetBlueprint, UWidget* Widget, const FString& NewDisplayName)
{
	FUMGWidgetInfo EmptyInfo;

	if (!WidgetBlueprint || !Widget)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("RenameWidget: WidgetBlueprint and Widget are required."));
		return EmptyInfo;
	}

	FText OutErrorMessage;
	if (!FWidgetBlueprintOperationUtils::VerifyWidgetRename(WidgetBlueprint, Widget, FText::FromString(NewDisplayName), OutErrorMessage))
	{
		UKismetSystemLibrary::RaiseScriptError(OutErrorMessage.ToString());
		return EmptyInfo;
	}

	if (!FWidgetBlueprintOperationUtils::RenameWidget(WidgetBlueprint, Widget, NewDisplayName))
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("RenameWidget: Failed to rename widget to '%s'."), *NewDisplayName));
		return EmptyInfo;
	}

	UPanelWidget* Parent = Widget->Slot ? Cast<UPanelWidget>(Widget->Slot->Parent) : nullptr;
	return Private::MakeWidgetInfo(Widget, Parent, WidgetBlueprint);
}

void UUMGToolSet::ToggleWidgetAsVariable(UWidgetBlueprint* WidgetBlueprint, UWidget* Widget, bool bIsVariable)
{
	if (!WidgetBlueprint || !Widget)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("ToggleWidgetAsVariable: WidgetBlueprint and Widget are required."));
		return;
	}

	FWidgetBlueprintOperationUtils::ToggleWidgetAsVariable(WidgetBlueprint, Widget, bIsVariable);
}

// ============================================================================
// UI Components
// ============================================================================

FUMGWidgetInfo UUMGToolSet::AddUIComponent(UWidgetBlueprint* WidgetBlueprint, FName WidgetName, TSubclassOf<UUIComponent> ComponentClass)
{	
	FUMGWidgetInfo EmptyInfo;
	
	if (!WidgetBlueprint || WidgetName.IsNone() || !ComponentClass)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("AddUIComponent: WidgetBlueprint, WidgetName, and ComponentClass are required."));
		return EmptyInfo;
	}

	FText OutErrorMessage;
	UWidget* Widget = FWidgetBlueprintOperationUtils::AddUIComponent(WidgetBlueprint, ComponentClass.Get(), WidgetName, OutErrorMessage);
	if (!Widget)
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("AddUIComponent: %s"), *OutErrorMessage.ToString()));
		return EmptyInfo;
	}

	FUMGWidgetInfo Info = Private::MakeWidgetInfo(Widget, Widget->GetParent(), WidgetBlueprint);
	if (UUIComponentWidgetBlueprintExtension* Ext = UWidgetBlueprintExtension::GetExtension<UUIComponentWidgetBlueprintExtension>(WidgetBlueprint))
	{
		for (UUIComponent* Component : Ext->GetComponentsFor(Widget))
		{
			if (Component)
			{
				FUMGUIComponentInfo CompInfo;
				CompInfo.Component = Component;
				CompInfo.ComponentClassPath = Private::GetResolvableClassPath(Component->GetClass());
				Info.UIComponents.Add(MoveTemp(CompInfo));
			}
		}
	}
	return Info;
}

bool UUMGToolSet::RemoveUIComponent(UWidgetBlueprint* WidgetBlueprint, FName WidgetName, TSubclassOf<UUIComponent> ComponentClass)
{
	if (!WidgetBlueprint || WidgetName.IsNone() || !ComponentClass)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("RemoveUIComponent: WidgetBlueprint, WidgetName, and ComponentClass are required."));
		return false;
	}

	FText OutErrorMessage;
	if (!FWidgetBlueprintOperationUtils::RemoveUIComponent(WidgetBlueprint, ComponentClass.Get(), WidgetName, OutErrorMessage))
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("RemoveUIComponent: %s"), *OutErrorMessage.ToString()));
		return false;
	}
	return true;
}

bool UUMGToolSet::MoveUIComponent(UWidgetBlueprint* WidgetBlueprint, FName WidgetName, TSubclassOf<UUIComponent> ComponentClassToMove, TSubclassOf<UUIComponent> RelativeToComponentClass, bool bMoveAfter)
{
	if (!WidgetBlueprint || WidgetName.IsNone() || !ComponentClassToMove || !RelativeToComponentClass)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("MoveUIComponent: WidgetBlueprint, WidgetName, ComponentClassToMove, and RelativeToComponentClass are required."));
		return false;
	}

	FText OutErrorMessage;
	if (!FWidgetBlueprintOperationUtils::MoveUIComponent(WidgetBlueprint, ComponentClassToMove.Get(), RelativeToComponentClass.Get(), WidgetName, bMoveAfter, OutErrorMessage))
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("MoveUIComponent: %s"), *OutErrorMessage.ToString()));
		return false;
	}
	return true;
}

// ============================================================================
// Lifecycle
// ============================================================================

bool UUMGToolSet::CompileWidgetBlueprint(UWidgetBlueprint* WidgetBlueprint)
{
	if (!WidgetBlueprint)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("CompileWidgetBlueprint: WidgetBlueprint is required."));
		return false;
	}

	// Clear bIsNewlyCreated so BindWidget validation reports errors (not just warnings).
	// The UMG compiler treats missing BindWidgets as warnings on newly-created BPs
	// (WidgetBlueprintCompiler.cpp:1284), which hides real errors from the AI.
	WidgetBlueprint->bIsNewlyCreated = false;

	// Compile — matches BlueprintEditorLibrary::CompileBlueprint pattern.
	FKismetEditorUtilities::CompileBlueprint(WidgetBlueprint, EBlueprintCompileOptions::SkipSave);

	// If compile succeeded, exit early (same check as BlueprintEditorLibrary)
	if (WidgetBlueprint->Status != BS_Error)
	{
		return true;
	}

	// Walk all graphs and collect error messages from nodes — same pattern as
	// BlueprintTools.compile_blueprint which uses ListNodesWithErrors.
	FString Errors;
	TArray<UEdGraph*> AllGraphs;
	AllGraphs.Append(WidgetBlueprint->UbergraphPages);
	AllGraphs.Append(WidgetBlueprint->FunctionGraphs);
	for (UEdGraph* Graph : AllGraphs)
	{
		if (!Graph) continue;
		for (UEdGraphNode* Node : Graph->Nodes)
		{
			if (Node && Node->bHasCompilerMessage && Node->ErrorType <= (int32)EMessageSeverity::Error)
			{
				if (!Errors.IsEmpty()) Errors += TEXT("; ");
				Errors += Node->ErrorMsg;
			}
		}
	}

	// Widget-specific: BindWidget errors come through the compiler MessageLog,
	// not graph nodes. Check the blueprint's message log for those.
	if (Errors.IsEmpty())
	{
		TSharedRef<IMessageLogListing> LogListing = FCompilerResultsLog::GetBlueprintMessageLog(WidgetBlueprint);
		const TArray<TSharedRef<FTokenizedMessage>>& Messages = LogListing->GetFilteredMessages();
		for (const TSharedRef<FTokenizedMessage>& Msg : Messages)
		{
			if (Msg->GetSeverity() <= EMessageSeverity::Error)
			{
				if (!Errors.IsEmpty()) Errors += TEXT("; ");
				Errors += Msg->ToText().ToString();
			}
		}
	}

	if (Errors.IsEmpty())
	{
		Errors = TEXT("Blueprint has compile errors.");
	}

	UKismetSystemLibrary::RaiseScriptError(FString::Printf(
		TEXT("CompileWidgetBlueprint: Compilation failed. %s"), *Errors));
	return false;
}

// ============================================================================
// Widget Description
// ============================================================================

FUMGWidgetDescriptionResult UUMGToolSet::GetWidgetDescription(const UWidgetBlueprint* WidgetBlueprint, const UWidget* StartWidget, int32 MaxDepth)
{
	if (!WidgetBlueprint)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("GetWidgetDescription: WidgetBlueprint is null."));
		return {};
	}

	if (!WidgetBlueprint->WidgetTree)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("GetWidgetDescription: WidgetBlueprint has no widget tree."));
		return {};
	}

	if (StartWidget && !StartWidget->IsIn(WidgetBlueprint->WidgetTree))
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("GetWidgetDescription: StartWidget does not belong to the given WidgetBlueprint."));
		return {};
	}

	if (MaxDepth < -1)
	{
		UE_LOG(LogUMGToolSet, Warning, TEXT("GetWidgetDescription: MaxDepth %d clamped to -1 (no limit)."), MaxDepth);
		MaxDepth = -1;
	}

	FUMGWidgetDescriptionResult Result;
	TStringBuilder<4096> Out;

	FWidgetBlueprintOperationUtils::WalkWidgetTree(
		WidgetBlueprint, StartWidget, MaxDepth,
		[&](const UWidget* Widget, const int32 Depth, const FName SlotName)
		{
			UPanelWidget* Parent = Widget ? Widget->GetParent() : nullptr;
			Result.Widgets.Add(Private::MakeWidgetInfo(
				const_cast<UWidget*>(Widget),
				Parent,
				const_cast<UWidgetBlueprint*>(WidgetBlueprint)));

			const int32 Index = Result.Widgets.Num() - 1;
			Private::RenderFullWidget(Out, Widget, Depth, SlotName, Index);
		});

	Result.Description = FString(Out);
	return Result;
}

int32 UUMGToolSet::GetWidgetTreeDepth(const UWidgetBlueprint* WidgetBlueprint, const UWidget* StartWidget)
{
	if (!WidgetBlueprint)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("GetWidgetTreeDepth: WidgetBlueprint is null."));
		return -1;
	}

	if (!WidgetBlueprint->WidgetTree)
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("GetWidgetTreeDepth: WidgetBlueprint has no widget tree."));
		return -1;
	}

	if (StartWidget && !StartWidget->IsIn(WidgetBlueprint->WidgetTree))
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("GetWidgetTreeDepth: StartWidget does not belong to the given WidgetBlueprint."));
		return -1;
	}

	return FWidgetBlueprintOperationUtils::ComputeWidgetTreeDepth(WidgetBlueprint, StartWidget);
}
