// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ToolsetRegistry/ToolsetDefinition.h"
#include "Blueprint/UserWidget.h"
#include "Components/Widget.h"
#include "Components/PanelWidget.h"
#include "Components/PanelSlot.h"
#include "Extensions/UIComponent.h"
#include "UMGToolSet.generated.h"

class UWidgetBlueprint;

// ============================================================================
// Structs
// ============================================================================

/** Info for a single UI component attached to a widget instance. */
USTRUCT(BlueprintType)
struct FUMGUIComponentInfo
{
	GENERATED_BODY()

	/**
	 * The component archetype instance. Use ObjectTools to read/write its properties.
	 *   1. ObjectTools.list_properties(Component) -> get exact property names
	 *   2. ObjectTools.get_properties(Component, [names]) -> read current values
	 *   3. ObjectTools.set_properties(Component, {name: value}) -> write values
	 */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	TObjectPtr<UUIComponent> Component = nullptr;

	/** Class path for this component. Pass directly to AddUIComponent, RemoveUIComponent, or MoveUIComponent as ComponentClass. */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	FSoftClassPath ComponentClassPath;
};

/**
 * Widget info with hierarchy pointers. Returned by all widget query and creation functions.
 * All TObjectPtr fields serialize as {"refPath": "..."} - pass them directly back to other functions.
 *
 * Reading properties:
 *   1. ObjectTools.list_properties(Widget) -> get exact property names
 *   2. ObjectTools.get_properties(Widget, [names]) -> read current values
 *   3. ObjectTools.set_properties(Widget, {name: value}) -> write values
 * Do the same for the Slot pointer to set padding, alignment, anchors, etc.
 * Get property names via ObjectTools.list_properties - they vary per widget class.
 *
 * Hierarchy context:
 *   - Parent set        -> widget was added as a child of this panel
 *   - NamedSlotHost set -> widget is named slot content on this host
 *   - Both null         -> root widget
 *   - bInherited true   -> defined in the C++ parent class, not user-created
 */
USTRUCT(BlueprintType)
struct FUMGWidgetInfo
{
	GENERATED_BODY()

	/** The widget instance. Pass to AddWidget as parent, ObjectTools for properties. */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	TObjectPtr<UWidget> Widget = nullptr;

	/** Parent panel widget. nullptr when this is the root widget or when this widget lives in a named slot (check NamedSlotHost instead). */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	TObjectPtr<UPanelWidget> Parent = nullptr;

	/** Panel slot for this widget. Pass to ObjectTools to read/write slot properties (padding, alignment, anchors, etc.). nullptr for root widget. */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	TObjectPtr<UPanelSlot> Slot = nullptr;

	/** Named slot host. If set, this widget is named slot content - use SetNamedSlotContent with this host instead of AddWidget. Call GetNamedSlots to find the slot name. */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	TObjectPtr<UWidget> NamedSlotHost = nullptr;

	/** Class path for this widget. Pass directly to AddWidget or SetNamedSlotContent WidgetClass param. */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	FSoftClassPath WidgetClassPath;

	/** Widget instance name. */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	FName WidgetName;

	/** Whether this widget is exposed as a blueprint variable. */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	bool bIsVariable = false;

	/** True if this widget is inherited from the C++ parent class. */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	bool bInherited = false;

	/** UI components attached to this widget, in display order. Pass ComponentClass entries back to AddUIComponent / RemoveUIComponent / MoveUIComponent. */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	TArray<FUMGUIComponentInfo> UIComponents;
};

/** Named slot binding entry. Returned by GetNamedSlots. */
USTRUCT(BlueprintType)
struct FUMGNamedSlotEntry
{
	GENERATED_BODY()

	/** Slot name (e.g., "content", "header"). */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	FName SlotName;

	/** Host widget that owns the slot. nullptr = inherited from parent class. */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	TObjectPtr<UWidget> HostWidget = nullptr;

	/** Root widget placed in the slot. */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	TObjectPtr<UWidget> ContentWidget = nullptr;
};

/** Blueprint-level info. Part of FUMGWidgetTreeInfo returned by GetWidgets. */
USTRUCT(BlueprintType)
struct FUMGWidgetBlueprintInfo
{
	GENERATED_BODY()

	/** Parent class of this widget blueprint. Pass directly to CreateWidgetBlueprint's ParentClass param. */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	TSubclassOf<UUserWidget> ParentClass;

	/** Root widget class. */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	TSubclassOf<UWidget> RootWidgetClass;

	/** Total widget count in the tree. */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	int32 WidgetCount = 0;

	/** Number of inherited widgets (from C++ parent class). */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	int32 InheritedWidgetCount = 0;

	/** Number of named slot bindings. */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	int32 NamedSlotCount = 0;
};

/** Combined result from GetWidgets: blueprint info + full widget tree. */
USTRUCT(BlueprintType)
struct FUMGWidgetTreeInfo
{
	GENERATED_BODY()

	/** Blueprint-level info: parent class, root widget class, counts. */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	FUMGWidgetBlueprintInfo Info;

	/** All widgets in depth-first order. Parent pointer indicates hierarchy. */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	TArray<FUMGWidgetInfo> Widgets;
};

/** Return value of GetWidgetDescription. */
USTRUCT(BlueprintType)
struct FUMGWidgetDescriptionResult
{
	GENERATED_BODY()

	/** Human-readable tree. Each line is prefixed [N] where N is the 0-based index into Widgets. */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	FString Description;

	/** Flat walk-order list. result.Widgets[N] corresponds to line [N] in Description. Same entries as GetWidgets. */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	TArray<FUMGWidgetInfo> Widgets;
};

/** Widget class entry for ListWidgetClasses and GetWidgetClassInfo. */
USTRUCT(BlueprintType)
struct FUMGWidgetClassEntry
{
	GENERATED_BODY()

	/** The widget class. */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	TSubclassOf<UWidget> WidgetClass;

	/** Whether this class is a panel (can have children). */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	bool bIsPanel = false;

	/** Category for this widget class. */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	FText Category;

	/** Description for this widget class that may contain more information on how the Widget is used and when to use it. */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	FText Description;
};

/**
 * One member (property or function) on the old widget class that has no compatible counterpart
 * on the new class, paired with a human-readable explanation: "no member of that name on the new
 * class" vs "name exists but the type/signature is incompatible".
 */
USTRUCT(BlueprintType)
struct FWidgetUnmatchedMember
{
	GENERATED_BODY()

	/** Name of the member on the old class. */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	FName Name;

	/** Reason this member couldn't be matched on the new class (missing name or type/signature mismatch). */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	FText Reason;
};

/**
 * Result of ReplaceWidgetWithTemplate. Carries success/error state plus the lists of public
 * members on the old widget that don't have a compatible counterpart on the new widget class.
 * Each list is split into "still referenced in the outer blueprint" (BP graphs, bindings,
 * animation property tracks, extension bindings) and the broader "all unmatched" set so
 * callers can warn the user about references that will dangle. Every entry carries the
 * per-member reason.
 */
USTRUCT(BlueprintType)
struct FWidgetReplacementReport
{
	GENERATED_BODY()

	/** True if the replacement was performed. */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	bool bSuccess = false;

	/** List of referenced properties/functions that are missing from the new class. */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	FText MissingReferencesWarning;

	/** Blueprint-visible properties on the old class that are missing or type-incompatible on the new class, with per-member reasons. */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	TArray<FWidgetUnmatchedMember> UnmatchedProperties;

	/** Subset of UnmatchedProperties that the outer blueprint references through this widget (graphs, bindings, animation property tracks). */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	TArray<FWidgetUnmatchedMember> UnmatchedReferencedProperties;

	/** BlueprintCallable / BlueprintEvent functions and multicast delegates on the old class that are missing or signature-incompatible on the new class, with per-member reasons. */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	TArray<FWidgetUnmatchedMember> UnmatchedFunctions;

	/** Subset of UnmatchedFunctions that the outer blueprint's graphs currently reference through this widget. */
	UPROPERTY(BlueprintReadOnly, Category = "UMGToolSet")
	TArray<FWidgetUnmatchedMember> UnmatchedReferencedFunctions;
};

// ============================================================================
// Toolset
// ============================================================================

/**
 * UMG widget toolset for AI-driven widget creation and tree manipulation.
 *
 * IMPORTANT WORKFLOW - for every widget and slot returned by this toolset:
 *   1. Call ObjectTools.list_properties(widget) to discover exact property names.
 *   2. Call ObjectTools.get_properties(widget, [...]) with those exact names.
 *   3. Call ObjectTools.set_properties(widget, {...}) with those exact names.
 * Property names vary per widget class and CANNOT be guessed - list_properties is required.
 * Skipping step 1 causes set_properties to silently fail or set wrong properties.
 *
 * Returns UObject pointers - ToolsetRegistry serializes them as {"refPath": "..."} automatically.
 * Pass returned Widget, Slot, and Parent pointers directly to ObjectTools or back to this toolset.
 */
UCLASS(BlueprintType)
class UUMGToolSet : public UToolsetDefinition
{
	GENERATED_BODY()

public:
	// ---- Creation ----

	/**
	 * Creates a new Widget Blueprint asset. Returns the blueprint or nullptr on failure.
	 * @param FolderPath Content folder path, e.g. "/Game/UI/Widgets".
	 * @param AssetName Name for the new blueprint asset.
	 * @param ParentClass The parent UUserWidget class. Get this from GetWidgets Info.ParentClass on the source blueprint.
	 */
	UFUNCTION(meta = (AICallable), Category = "UMG|Creation")
	static UWidgetBlueprint* CreateWidgetBlueprint(const FString& FolderPath, const FString& AssetName, TSubclassOf<UUserWidget> ParentClass);

	/**
	 * Adds a widget to the tree at the specified position. Returns full widget info including Slot pointer.
	 * When ParentWidget is null and no root exists, the new widget becomes the root of the tree.
	 * Use ObjectTools.list_properties on the returned Widget and Slot to get property names before calling set_properties.
	 * @param WidgetBlueprint The widget blueprint to modify.
	 * @param WidgetClass The widget class to instantiate.
	 * @param WidgetDisplayName Display name for the new widget instance.
	 * @param ParentWidget The panel widget to add to. Pass null to add to root, or to make this widget the root if the tree is empty.
	 * @param ChildIndex Position in parent's child list (0 = first child). -1 (default) appends to end.
	 */
	UFUNCTION(meta = (AICallable), Category = "UMG|Creation")
	static FUMGWidgetInfo AddWidget(UWidgetBlueprint* WidgetBlueprint, TSubclassOf<UWidget> WidgetClass, const FString& WidgetDisplayName, UWidget* ParentWidget = nullptr, int32 ChildIndex = -1);

	/**
	 * Sets content for a named slot. Returns full widget info including Slot pointer.
	 * @param WidgetBlueprint The widget blueprint to modify.
	 * @param HostWidget The widget that owns the named slot, or null to target the root WidgetTree.
	 * @param SlotName Name of the slot to fill (e.g., "content", "header").
	 * @param WidgetClass The widget class to place in the slot.
	 * @param WidgetName Name for the new widget instance.
	 */
	UFUNCTION(meta = (AICallable), Category = "UMG|Creation")
	static FUMGWidgetInfo SetNamedSlotContent(UWidgetBlueprint* WidgetBlueprint, UWidget* HostWidget, FName SlotName, TSubclassOf<UWidget> WidgetClass, FName WidgetName);

	// ---- Query ----

	/**
	 * Returns blueprint info and all widgets in depth-first order.
	 * Children within each parent are in their panel slot order - this is the hierarchy order shown in the designer.
	 * Info contains ParentClass (pass to CreateWidgetBlueprint) and RootWidgetClass.
	 * Use ObjectTools.list_properties on each returned Widget and Slot to get property names before calling set_properties.
	 * @param WidgetBlueprint The widget blueprint asset (e.g. "/Game/UI/WBP_MyWidget"), excluding the "_C" suffix.
	 */
	UFUNCTION(meta = (AICallable), Category = "UMG|Query")
	static FUMGWidgetTreeInfo GetWidgets(UWidgetBlueprint* WidgetBlueprint);

	/**
	 * Returns named slot bindings (separate from tree hierarchy).
	 * @param WidgetBlueprint The widget blueprint to query.
	 */
	UFUNCTION(meta = (AICallable), Category = "UMG|Query")
	static TArray<FUMGNamedSlotEntry> GetNamedSlots(UWidgetBlueprint* WidgetBlueprint);

	/**
	 * Lists widget blueprints in a content folder.
	 * @param FolderPath Content folder to search, e.g. "/Game/UI". Searches recursively.
	 */
	UFUNCTION(meta = (AICallable), Category = "UMG|Query")
	static TArray<FSoftObjectPath> ListWidgetBlueprints(const FString& FolderPath);

	/**
	 * Lists available widget classes, optionally filtered by name substring.
	 * @param Filter Substring to match against class names. Pass empty string to return all classes.
	 */
	UFUNCTION(meta = (AICallable), Category = "UMG|Query")
	static TArray<FUMGWidgetClassEntry> ListWidgetClasses(const FString& Filter = TEXT(""));

	/**
	 * Returns the Category, Description and if it's a Panel for a single widget class.
	 * Same per-entry data as ListWidgetClasses, but lets callers query a class they already have
	 * without scanning every UClass. Returns an empty entry if WidgetClass is null.
	 * Can be used to get more information on the class from the Description and Category.
	 * @param WidgetClass The widget class to inspect.
	 */
	UFUNCTION(meta = (AICallable), Category = "UMG|Query")
	static FUMGWidgetClassEntry GetWidgetClassInfo(TSubclassOf<UWidget> WidgetClass);

	// ---- Modification ----

	/**
	 * Moves a widget to a new parent panel at the specified position. Returns updated widget info with new Slot.
	 * @param WidgetBlueprint The widget blueprint to modify.
	 * @param Widget The widget to move.
	 * @param NewParent The destination panel widget.
	 * @param ChildIndex Position in new parent's child list (0 = first child). -1 (default) appends to end.
	 */
	UFUNCTION(meta = (AICallable), Category = "UMG|Modification")
	static FUMGWidgetInfo MoveWidget(UWidgetBlueprint* WidgetBlueprint, UWidget* Widget, UPanelWidget* NewParent, int32 ChildIndex = -1);

	/**
	 * Removes a widget and its children from the tree.
	 * @param WidgetBlueprint The widget blueprint to modify.
	 * @param Widget The widget to remove (along with its children).
	 */
	UFUNCTION(meta = (AICallable), Category = "UMG|Modification")
	static bool RemoveWidget(UWidgetBlueprint* WidgetBlueprint, UWidget* Widget);

	/**
	 * Renames a widget. Returns updated widget info or empty on failure.
	 * @param WidgetBlueprint The widget blueprint to modify.
	 * @param Widget The widget to rename.
	 * @param NewDisplayName The new display name.
	 */
	UFUNCTION(meta = (AICallable), Category = "UMG|Modification")
	static FUMGWidgetInfo RenameWidget(UWidgetBlueprint* WidgetBlueprint, UWidget* Widget, const FString& NewDisplayName);

	/**
	 * Sets the bIsVariable flag.
	 * @param WidgetBlueprint The widget blueprint to modify.
	 * @param Widget The widget to update.
	 * @param bIsVariable True to expose as a blueprint variable, false to hide it.
	 */
	UFUNCTION(meta = (AICallable), Category = "UMG|Modification")
	static void ToggleWidgetAsVariable(UWidgetBlueprint* WidgetBlueprint, UWidget* Widget, bool bIsVariable);

	/**
	 * Adds a Blueprint event handler graph node bound to a widget's multicast delegate event,
	 *
	 * Typical events: UButton::OnClicked / OnPressed / OnReleased / OnHovered / OnUnhovered,
	 * UCheckBox::OnCheckStateChanged, USlider::OnValueChanged. The matching delegate
	 * UPROPERTY must exist on PropertyClass (or a parent of it).
	 *
	 * Preconditions:
	 *   - PropertyName must exist in the blueprint.
	 *   - PropertyClass must be the widget's class (or a parent class) that declares the delegate.
	 *
	 * @param WidgetBlueprint The widget blueprint that will own the event handler.
	 * @param EventName       Name of the multicast delegate UPROPERTY on PropertyClass (e.g. "OnClicked").
	 * @param PropertyName    Name of the blueprint variable owning the event.
	 * @param PropertyClass   Class declaring the delegate, typically the widget's class (e.g. UButton::StaticClass()).
	 */
	UFUNCTION(meta = (AICallable), Category = "UMG|Modification")
	static bool BindToEventProperty(UWidgetBlueprint* WidgetBlueprint, const FName EventName, FName PropertyName, UClass* PropertyClass);

	/**
	 * Wraps one or more widgets in a new panel widget of the specified class.
	 * Only the root-most widgets in the selection are wrapped — children of other selected widgets
	 * are skipped because their parent will be wrapped. Returns info for each newly created wrapper.
	 *
	 * Use ObjectTools.list_properties on each returned Widget and Slot to discover property names
	 * before calling set_properties (padding, alignment, anchors, etc. vary per panel class).
	 *
	 * @param WidgetBlueprint The widget blueprint to modify.
	 * @param Widgets         The widgets to wrap. Must all be in WidgetBlueprint's tree.
	 * @param WrapperClass    The panel widget class to wrap with (must be a UPanelWidget subclass).
	 */
	UFUNCTION(meta = (AICallable), Category = "UMG|Modification")
	static TArray<FUMGWidgetInfo> WrapWidgets(UWidgetBlueprint* WidgetBlueprint, TArray<UWidget*> Widgets, TSubclassOf<UPanelWidget> WrapperClass);

	// ---- Widget Description ----

	/**
	 * Full property dump of every widget in the tree.
	 * Each line: [N] Type Name  Prop:Value ...  slot:(SlotProp:Value ...)
	 * N is the 0-based index into result.Widgets -- use result.Widgets[N] to get the widget ref without text parsing.
	 *
	 * Same indentation format as GetTaggedWidgetDescription; richer per-widget detail.
	 *
	 * @param WidgetBlueprint  The WBP asset.
	 * @param StartWidget      nullptr = full tree from root.
	 * @param MaxDepth         -1 = no limit; 0 = StartWidget only; N = N levels.
	 */
	UFUNCTION(meta = (AICallable), Category = "UMGToolSet|WidgetDescription")
	static FUMGWidgetDescriptionResult GetWidgetDescription(const UWidgetBlueprint* WidgetBlueprint, const UWidget* StartWidget = nullptr, int32 MaxDepth = -1);

	/**
	 * Returns the maximum depth of the widget tree. Depth: root with no children = 0; root + children = 1; etc.
	 *
	 * @return Depth as int32; -1 on error.
	 */
	UFUNCTION(meta = (AICallable), Category = "UMG|WidgetDescription")
	static int32 GetWidgetTreeDepth(
		const UWidgetBlueprint* WidgetBlueprint,
		const UWidget* StartWidget = nullptr);


	// ---- UI Components ----

	/**
	 * Adds a UI component of the given class to the named widget.
	 * @param WidgetBlueprint The widget blueprint to modify.
	 * @param WidgetName Name of the widget instance to add the component to.
	 * @param ComponentClass The UIComponent subclass to add.
	 * @return Info for the widget the component was added to, including the populated UIComponents array. Returns an empty info on failure.
	 */
	UFUNCTION(meta = (AICallable), Category = "UMG|Components")
	static FUMGWidgetInfo AddUIComponent(UWidgetBlueprint* WidgetBlueprint, FName WidgetName, TSubclassOf<UUIComponent> ComponentClass);

	/**
	 * Removes a UI component of the given class from the named widget.
	 * @param WidgetBlueprint The widget blueprint to modify.
	 * @param WidgetName Name of the widget instance to remove the component from.
	 * @param ComponentClass The UIComponent subclass to remove.
	 * @return True if the component was removed, false if it was not found.
	 */
	UFUNCTION(meta = (AICallable), Category = "UMG|Components")
	static bool RemoveUIComponent(UWidgetBlueprint* WidgetBlueprint, FName WidgetName, TSubclassOf<UUIComponent> ComponentClass);

	/**
	 * Moves a UI component before or after another component on the same widget.
	 * @param WidgetBlueprint The widget blueprint to modify.
	 * @param WidgetName Name of the widget instance whose component to move.
	 * @param ComponentClassToMove The UIComponent subclass to reorder.
	 * @param RelativeToComponentClass The UIComponent subclass to move relative to.
	 * @param bMoveAfter True to place after RelativeToComponentClass, false to place before.
	 * @return True if the component was moved, false if either component was not found.
	 */
	UFUNCTION(meta = (AICallable), Category = "UMG|Components")
	static bool MoveUIComponent(UWidgetBlueprint* WidgetBlueprint, FName WidgetName, TSubclassOf<UUIComponent> ComponentClassToMove, TSubclassOf<UUIComponent> RelativeToComponentClass, bool bMoveAfter);

	/**
	 * Replaces a widget instance in the blueprint's widget tree with a new instance created from a
	 * different template widget class. Preserves references for members that exist on both classes
	 * with a compatible type/signature: bindings, BP graph variable references, animation
	 * bindings, and delegate bindings. Members without a compatible counterpart on the new class
	 * are listed in the returned report; references to those members in the outer blueprint will
	 * become orphaned graph nodes / dangling bindings.
	 *
	 * @param WidgetBlueprint    The widget blueprint containing the widget to replace.
	 * @param WidgetToReplace    The widget instance to replace (must be in WidgetBlueprint's tree).
	 * @param TemplateClass      The widget class to create the replacement from.
	 * @return Report whose bSuccess flag indicates whether the replacement happened, MissingReferencesWarning
	 *         describes warnings that need action, and the Unmatched* arrays list members that have no
	 *         compatible counterpart on the new class (with the *Referenced* subsets being the
	 *         ones the blueprint actually uses today).
	 */
	UFUNCTION(meta = (AICallable), Category = "UMG|Modification")
	static FWidgetReplacementReport ReplaceWidgetWithTemplate(UWidgetBlueprint* WidgetBlueprint, UWidget* WidgetToReplace, TSubclassOf<UWidget> TemplateClass);

	/**
	 * Replaces a host widget with the content of one of its named slots. The host must implement
	 * INamedSlotInterface (e.g., a UUserWidget exposing named slots). The slot's content widget is
	 * moved up to take the host's place in the tree.
	 *
	 * @param WidgetBlueprint    The widget blueprint containing the host widget.
	 * @param WidgetToReplace    The host widget to replace.
	 * @param NamedSlot          The slot whose content replaces WidgetToReplace.
	 * @return true on success, false if validation failed.
	 */
	UFUNCTION(meta = (AICallable), Category = "UMG|Modification")
	static bool ReplaceWidgetWithNamedSlot(UWidgetBlueprint* WidgetBlueprint, UWidget* WidgetToReplace, FName NamedSlot);

	/**
	 * Replaces a panel widget with its first child, removing the panel from the tree.
	 * The widget to replace must be a UPanelWidget with only one child.
	 *
	 * @param WidgetBlueprint    The widget blueprint containing the panel widget.
	 * @param WidgetToReplace    The panel widget to replace with its first child.
	 * @return true on success, false if validation failed.
	 */
	UFUNCTION(meta = (AICallable), Category = "UMG|Modification")
	static bool ReplaceWidgetWithChild(UWidgetBlueprint* WidgetBlueprint, UWidget* WidgetToReplace);

	// ---- Lifecycle ----

	/**
	 * Compiles a widget blueprint. Returns false with error details if compilation fails.
	 * Errors include missing BindWidget bindings, type mismatches, and graph errors.
	 * Call after all widgets and properties are set. Save separately via AssetTools.save_asset.
	 * @param WidgetBlueprint The widget blueprint to compile.
	 */
	UFUNCTION(meta = (AICallable), Category = "UMG|Lifecycle")
	static bool CompileWidgetBlueprint(UWidgetBlueprint* WidgetBlueprint);
};
