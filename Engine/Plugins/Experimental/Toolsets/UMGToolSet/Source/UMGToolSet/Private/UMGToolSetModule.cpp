// Copyright Epic Games, Inc. All Rights Reserved.

#include "UMGToolSetModule.h"
#include "Misc/CoreDelegates.h"
#include "ToolsetRegistry/UToolsetRegistry.h"
#include "UMGToolSet.h"

#define LOCTEXT_NAMESPACE "UMGToolSet"

void FUMGToolSetModule::StartupModule()
{
	FCoreDelegates::OnAllModuleLoadingPhasesComplete.AddRaw(
		this, &FUMGToolSetModule::OnAllModuleLoadingPhasesComplete);
	FCoreDelegates::OnPreExit.AddRaw(
		this, &FUMGToolSetModule::OnPreExit);
}

void FUMGToolSetModule::ShutdownModule()
{
	FCoreDelegates::OnAllModuleLoadingPhasesComplete.RemoveAll(this);
	FCoreDelegates::OnPreExit.RemoveAll(this);
}

void FUMGToolSetModule::OnAllModuleLoadingPhasesComplete()
{
	UToolsetRegistry::RegisterToolsetClass(UUMGToolSet::StaticClass());
}

void FUMGToolSetModule::OnPreExit()
{
	UToolsetRegistry::UnregisterToolsetClass(UUMGToolSet::StaticClass());
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FUMGToolSetModule, UMGToolSet)
