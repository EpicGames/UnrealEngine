// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

/**
 * UMGToolSet module - provides AI assistant toolsets for UMG widget manipulation.
 */
class FUMGToolSetModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	/** Called when all modules have finished loading */
	void OnAllModuleLoadingPhasesComplete();

	/** Called before application exit */
	void OnPreExit();
};
