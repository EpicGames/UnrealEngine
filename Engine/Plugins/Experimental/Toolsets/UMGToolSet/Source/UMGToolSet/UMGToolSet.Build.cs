// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UMGToolSet : ModuleRules
{
	public UMGToolSet(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"ToolsetRegistry",
				"UMG",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"UnrealEd",
				"Slate",
				"SlateCore",
				"UMGEditor",
				"Kismet",
				"AssetRegistry",
				"MessageLog",
				"BlueprintGraph",
				"MovieScene",
				"MovieSceneTracks",
			}
		);
	}
}
