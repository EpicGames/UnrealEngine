{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "UMG ToolSet",
	"Description": "AI assistant tools for creating and manipulating UMG widgets via reflection.",
	"Category": "UI",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EditorOnly": true,
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "UMGToolSet",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"TargetAllowList": ["Editor"]
		}
	],
	"Plugins": [
		{
			"Name": "ToolsetRegistry",
			"Enabled": true
		}
	]
}
